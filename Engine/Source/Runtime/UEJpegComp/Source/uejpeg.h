// Copyright Epic Games, Inc. All Rights Reserved.
/*

UEJPEG - A JPEG Super-compressor using an oodle backend as the entropy coder.

Can encode from a JPEG file to an UEJPEG file, and can decode directly to a JPEG file or a raw image.
Can also encode raw images directly to UEJPEG files.

API has simple APIs and threaded APIs. 

The simple APIs is as follows:
-----------------------------

The following functions encode from a JPEG file to a UEJPEG file...

  int uejpeg_encode_jpeg_mem(const unsigned char *jpeg_data, int jpeg_data_size, unsigned char **out_uejpeg_data, int *out_uejpeg_size, int flags);

The following functions encode from a raw RGBA image to a UEJPEG file...

  int uejpeg_encode_to_mem(unsigned char **out_data, int *out_size, int width, int height, int channels, const unsigned char *rgba, int quality, int flags);

The following functions decode from a UEJPEG file back to a JPEG file...

  int uejpeg_decode_mem_to_jpeg(const unsigned char *data, int size, unsigned char **out_data, int *out_size, int flags);

The following functions decode from a UEJPEG file to a raw RGBA image...

  unsigned char *uejpeg_decode_mem(const unsigned char *data, int size, int *width, int *height, int *channels, int flags);

The threaded APIs is as follows:
-------------------------------

The following functions encode from a JPEG file to a UEJPEG file...

  uejpeg_encode_context_t uejpeg_encode_jpeg_mem_threaded_start(const unsigned char *data, int size, int flags);
  int uejpeg_encode_jpeg_thread_run(uejpeg_encode_context_t *context, int job_idx);
  int uejpeg_encode_jpeg_mem_threaded_finish(uejpeg_encode_context_t *context, unsigned char **out_data, int *out_size);

The following functions encode from a raw RGBA image to a UEJPEG file...

  uejpeg_encode_image_context_t uejpeg_encode_image_mem_threaded_start(int width, int height, int comp, const unsigned char *in_data, int quality, int flags);
  uejpeg_encode_image_context_t uejpeg_encode_image_threaded_start(const uejpeg_io_callbacks_t *io, void *io_user, int width, int height, int comp, const unsigned char *in_data, int quality, int flags);
  int uejpeg_encode_image_thread_run(uejpeg_encode_image_context_t *ctx, int job_idx);
  int uejpeg_encode_image_threaded_finish(uejpeg_encode_image_context_t *ctx);
  int uejpeg_encode_image_mem_threaded_finish(uejpeg_encode_image_context_t *ctx, unsigned char **out_data, int *out_size);

The following functions decode from a UEJPEG file to a raw RGBA image...

  uejpeg_decode_context_t uejpeg_decode_mem_threaded_start(const unsigned char *data, int size, int flags);
  uejpeg_decode_context_t uejpeg_decode_threaded_start(const uejpeg_io_callbacks_t *io, void *io_user, int flags);
  int uejpeg_decode_thread_run(uejpeg_decode_context_t *context, int job_idx);
  unsigned char *uejpeg_decode_threaded_finish(uejpeg_decode_context_t *context, int *out_width, int *out_height, int *out_comp);

You use the threaded APIs by first calling the _start function, which gives you a context. You then pass that context into the thread_run function N times,
where N is the value of the context.jobs_to_run variable. You then wait for all the threads to finish, and then call the _finish function to get the output data.

Example:

      uejpeg_encode_context_t ctx = uejpeg_encode_jpeg_mem_threaded_start(input, output, flags);
      #pragma omp parallel for
      for(int i = 0; i < ctx.jobs_to_run; ++i) {
        uejpeg_encode_jpeg_thread_run(&ctx, i);
      }
      uejpeg_encode_jpeg_file_threaded_finish(&ctx);

Other Notes:

* Custom Allocators can be set via uejpeg_set_alloc() function.
* Custom compression/decompression (must be set the same in both the encoder and decoder) can be set via uejpeg_set_compression() function.

*/
#pragma once

#ifdef __cplusplus
extern "C" {
#endif

// Error codes
typedef enum {
  UEJPEG_ERR_NONE                = 0, // No Error, everything OK! :)
  UEJPEG_ERR_NOT_A_JPEG          = 1, // Not a JPEG file. Happens if you try to encode from a non-JPEG file.
  UEJPEG_ERR_CANNOT_OPEN_FILE    = 2, // Cannot open file. Happens if the file routines cannot open the files you specified!
  UEJPEG_ERR_NOT_UEJPEG_FILE     = 3, // Not a uejpeg file. Happens if you try to decode a non-uejpeg file.
  UEJPEG_ERR_CORRUPT_UEJPEG_FILE = 4, // Corrupt uejpeg file. Happens if the uejpeg file is corrupt.
  UEJPEG_ERR_WRITE_FAILED        = 5, // Write failed. Happens if the file routines cannot write to the files you specified!
  UEJPEG_ERR_OUT_OF_MEMORY       = 6, // Out of memory. Happens if the uejpeg library cannot allocate enough memory.
  UEJPEG_ERR_UNSUPPORTED_COLORSPACE = 7, // Unsupported colorspace. Happens when exporting if the JPEG file is not YCbCr.
  UEJPEG_ERR_READ_FAILED         = 8, // Read failed. Happens if the file routines cannot read from the files you specified!
  UEJPEG_ERR_UNSUPPORTED         = 9, // Other Unsupported. Happens usually when you try to decode to jpeg in an uncommon format.
} uejpeg_error_t;

// Flags to be used in compression or decompression
typedef enum {
  UEJPEG_FLAG_NONE            = 0x0, // no flags
  UEJPEG_FLAG_NO_COMPRESSION  = 0x1, // don't compress the coefficients, just store them
  UEJPEG_FLAG_FASTDCT         = 0x2, // use a fast dct transform which matches libjpegturbo's fastdct.
} uejpeg_flags_t;

enum {
  UEJPEG_MAX_SPLITS = 16,
};

// IO callbacks
//  For use with uejpeg_*() to specify any input or output method. All methods internally use this!
typedef struct {
  size_t (*write)(void *user, const void *data, size_t size);  // write 'data' with 'size' bytes.  return number of bytes actually written
  size_t (*read)(void *user, void *data, size_t size);  // read 'data' with 'size' bytes.  return number of bytes actually read
  int (*eof)(void *user);  // return 1 if end-of-file, 0 otherwise
} uejpeg_io_callbacks_t;

// For using memory buffers as input or output.
typedef struct {
  char *base;
  char *ptr;
  size_t size;
} uejpeg_io_buffer_t;

extern const uejpeg_io_callbacks_t s_uejpeg_buffer_fns;

// For setting custom memory allocators.
typedef struct {
  void *(*alloc)(size_t size);  // allocate 'size' bytes.  return pointer to allocated memory
  void (*free)(void *ptr);  // free memory at 'ptr'
  void *(*realloc)(void *ptr, size_t size);  // reallocate 'ptr' to 'size' bytes.  return pointer to reallocated memory
} uejpeg_alloc_t;

typedef struct {
    void *data;
    size_t size;
} uejpeg_buffer_t;

// compression callbacks, for using custom compression methods, such as zlib
typedef struct {
  uejpeg_buffer_t (*compress)(const void *data, size_t size);
  int (*decompress)(const void *data, size_t size, void *out_data, size_t out_size);
} uejpeg_compression_t;

// Set custom allocators.  If not set, malloc/free/realloc will be used.
void uejpeg_set_alloc(const uejpeg_alloc_t *alloc);
// Set custom compression methods.  If not set, oodle compression methods will be used.
void uejpeg_set_compression(const uejpeg_compression_t *compression);

// An internal structure used to store the header of a uejpeg file.
typedef struct uejpeg_ihdr_t {
  unsigned version;
  unsigned width;
  unsigned height;
  unsigned char bit_depth;
  unsigned char comp;
  unsigned char method;
  unsigned char num_splits;
} uejpeg_ihdr_t;

// An internal structure used to store the header of a uejpeg file.
typedef struct uejpeg_lossy_hdr_t {
  unsigned short coef_dims[4][2];
  // Quantization tables
  unsigned short dequant[4][64];
  char app14_color_transform; // valid values are 0(CMYK),1(YCCK),2(YCbCrA)
  char jfif;
  char comp_id[4];
} uejpeg_lossy_hdr_t;

typedef struct {
  int y_start[4];
  int y_end[4];
  int width[4];
  int height[4];
  int num_blocks;
  short *xcoefs;
  char *split_xcoefs;
  unsigned char *oodle_buf;
  int oodle_size;
  int coefs_size;
} uejpeg_internal_split_t;

typedef struct {
  int error;
  int split_height_min;
  int split_height[4];
  int split_blocks[4];
  int total_split_blocks;
  uejpeg_internal_split_t splits[UEJPEG_MAX_SPLITS];
} uejpeg_internal_splits_t;

typedef struct {
  int error; // If there was an error, it is stored here.
  int jobs_to_run; // How many times to run uejpeg_decode_thread_run!

  // arguments
  const uejpeg_io_callbacks_t *io;
  void *io_user;
  int flags;

  // internal state
  int width, height, comp;
  int num_splits;

  uejpeg_lossy_hdr_t lhdr;
  uejpeg_internal_splits_t s;
} uejpeg_encode_context_t;

typedef struct {
  int error; // If there was an error, it is stored here.
  int jobs_to_run; // How many times to run uejpeg_decode_thread_run!

  // arguments
  int flags;

  // internal state
  uejpeg_ihdr_t ihdr;
  uejpeg_lossy_hdr_t lhdr;
  uejpeg_internal_splits_t spx;
  int yuva_stride[4];
  int yuva_height[4];
  unsigned char *yuva[4];
} uejpeg_decode_context_t;

typedef struct {
  int error; // If there was an error, it is stored here.
  int jobs_to_run; // How many times to run uejpeg_decode_thread_run!

  // arguments
  const uejpeg_io_callbacks_t *io;
  void *io_user;
  int width, height, comp;
  const unsigned char *in_data;
  int flags;

  // internal state
  int num_splits;
  uejpeg_internal_splits_t spx;
  uejpeg_lossy_hdr_t lhdr;
	float fdtbl_Y[64];
  float fdtbl_UV[64];
} uejpeg_encode_image_context_t;

// Encode a JPEG file to uejpeg format.
// This second function operates totally in memory.
int uejpeg_encode_jpeg_mem(const unsigned char *jpeg_data, int jpeg_data_size, unsigned char **out_uejpeg_data, int *out_uejpeg_size, int flags);

// Threaded API
// In this API you first call start, and it does some synchronous work and gives you a state context to pass to the run function.
// Then inside threads, you call the run function with job_idx == 0 .. context.jobs_to_run
// Then you wait for all the threads to finish processing all the jobs.
// Then you call finish to get the output data.
uejpeg_encode_context_t uejpeg_encode_jpeg_mem_threaded_start(const unsigned char *data, int size, int flags);
int uejpeg_encode_jpeg_thread_run(uejpeg_encode_context_t *context, int job_idx);
int uejpeg_encode_jpeg_mem_threaded_finish(uejpeg_encode_context_t *context, unsigned char **out_data, int *out_size);

// Encode a raw image to a uejpeg file.
int uejpeg_encode_to_mem(unsigned char **out_data, int *out_size, int width, int height, int channels, const unsigned char *rgba, int quality, int flags);

uejpeg_encode_image_context_t uejpeg_encode_image_mem_threaded_start(int width, int height, int comp, const unsigned char *in_data, int quality, int flags);
uejpeg_encode_image_context_t uejpeg_encode_image_threaded_start(const uejpeg_io_callbacks_t *io, void *io_user, int width, int height, int comp, const unsigned char *in_data, int quality, int flags);
int uejpeg_encode_image_thread_run(uejpeg_encode_image_context_t *ctx, int job_idx);
int uejpeg_encode_image_threaded_finish(uejpeg_encode_image_context_t *ctx);
int uejpeg_encode_image_mem_threaded_finish(uejpeg_encode_image_context_t *ctx, unsigned char **out_data, int *out_size);

// Decode a UEJPEG file to a raw image.
// Interface is like stbi_load_from_file, stbi_load_from_memory, stbi_load, etc.
unsigned char *uejpeg_decode_mem(const unsigned char *data, int size, int *width, int *height, int *channels, int flags);
unsigned char *uejpeg_decode(const uejpeg_io_callbacks_t *io, void *io_user, int *out_width, int *out_height, int *out_comp, int flags);

// Threaded API
// In this API you first call start, and it does some synchronous work and gives you a state context to pass to the run function.
// Then inside threads, you call the run function with job_idx == 0 .. context.jobs_to_run
// Then you wait for all the threads to finish processing all the jobs.
// Then you call finish to get the output data.

uejpeg_decode_context_t uejpeg_decode_mem_threaded_start(const unsigned char *data, int size, int flags);
uejpeg_decode_context_t uejpeg_decode_threaded_start(const uejpeg_io_callbacks_t *io, void *io_user, int flags);
int uejpeg_decode_thread_run(uejpeg_decode_context_t *context, int job_idx);
unsigned char *uejpeg_decode_threaded_finish(uejpeg_decode_context_t *context, int *out_width, int *out_height, int *out_comp);

// Decode a UEJPEG file to a JPEG file.
int uejpeg_decode_to_jpeg(const uejpeg_io_callbacks_t *in_io, void *in_io_user, const uejpeg_io_callbacks_t *out_io, void *out_io_user, int flags);
int uejpeg_decode_mem_to_jpeg(const unsigned char *data, int size, unsigned char **out_data, int *out_size, int flags);

#ifdef __cplusplus
}
#endif

