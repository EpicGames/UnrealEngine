// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System.IO;

public class UEJpegComp : ModuleRules
{
	public UEJpegComp(ReadOnlyTargetRules Target) : base(Target)
	{
		Type = ModuleType.External;

		bool bWithLib = false;
		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib");
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uejpeg_w64.lib"));
			bWithLib = true;
		}
		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.WindowsPlatform.Architecture == UnrealArch.Arm64)
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib");
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uejpeg_warm64.lib"));
			bWithLib = true;
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib");
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uejpeg_mac.a"));
			bWithLib = true;
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib/Unix", Target.Architecture.LinuxName);
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uejpeg_linux.a"));
			bWithLib = true;
		}

		if (bWithLib)
		{
			PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "Source"));
		}

		PublicDefinitions.Add("WITH_UEJPEG=" + (bWithLib ? '1' : '0'));
	}
}
